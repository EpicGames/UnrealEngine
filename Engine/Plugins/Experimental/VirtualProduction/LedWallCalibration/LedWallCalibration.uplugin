{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Led Wall Calibration",
	"Description": "Tools for Led Wall calibration",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "LedWallCalibrationEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		},
		{
			"Name": "LedWallCalibration",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			]
		}
	],
	"Plugins": [
		{
			"Name": "CameraCalibrationCore",
			"Enabled": true
		},
		{
			"Name": "OpenCV",
			"Enabled": true
		}
	]
}
