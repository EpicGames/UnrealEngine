// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LedWallCalibration : ModuleRules
{
	public LedWallCalibration(ReadOnlyTargetRules Target) : base(Target)
	{
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CameraCalibrationCore",
				"Core",
				"CoreUObject",
				"Engine",
				"MeshDescription",
				"OpenCV",
				"OpenCVHelper",
				"StaticMeshDescription",
			}
		);
	}
}
