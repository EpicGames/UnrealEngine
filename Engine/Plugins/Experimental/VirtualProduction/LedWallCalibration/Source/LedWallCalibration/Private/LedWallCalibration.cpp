// Copyright Epic Games, Inc. All Rights Reserved.

#include "LedWallCalibration.h"

#include "CalibrationPointComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/SCS_Node.h"
#include "Engine/StaticMesh.h"
#include "LedWallArucoGenerationOptions.h"
#include "LedWallCalibrationLog.h"
#include "StaticMeshAttributes.h"


#if WITH_OPENCV
// IWYU pragma: begin_keep
#include "OpenCVHelper.h"
#include "PreOpenCVHeaders.h"
#include "opencv2/aruco.hpp"
#include "PostOpenCVHeaders.h"
// IWYU pragma: end_keep
#endif //WITH_OPENCV

#define LOCTEXT_NAMESPACE "LedWallCalibration"

#if WITH_OPENCV
#define BUILD_ARUCO_INFO(x) { FString(TEXT(#x)), cv::aruco::getPredefinedDictionary(cv::aruco::x) }
#endif //WITH_OPENCV

namespace LedWallCalibration
{
	/**
	 * Structure to keep the positional information needed to generate the Arucos.
	 */
	struct FWallPanel
	{
		/** 3d corners of this panel */
		FVector Vertices[4];

		/** UV (2d) corners of this panel. Expected to be sorted top-left, top-right, bottom-left, bottom-right */
		FVector2f UVs[4];

		/** Estimated row number (zero based) of this panel in the wall it belongs to */
		int32 Row = 0;

		/** Estimated column number (zero based) of this panel in the wall it belongs to */
		int32 Col = 0;

		/** Sorts UVs and corresponding Vertices member variables by UV values */
		void SortByUVs()
		{
			TArray<int32, TInlineAllocator<4>> Indexes = { 0, 1, 2, 3 };

			Indexes.Sort([&](const int32& IdxA, const int32& IdxB) -> bool
				{
					// This row factor should be ok because we're sorting UVs that are not expected to go over 1.0.
					constexpr float UVYFactor = 100.0f; 

					const float ValueA = UVs[IdxA].X + UVYFactor * UVs[IdxA].Y;
					const float ValueB = UVs[IdxB].X + UVYFactor * UVs[IdxB].Y;

					return ValueA < ValueB;
				});

			FVector VerticesCopy[4];
			FVector2f UVsCopy[4];

			for (int32 Idx = 0; Idx < 4; ++Idx)
			{
				VerticesCopy[Idx] = Vertices[Idx];
				UVsCopy[Idx] = UVs[Idx];
			}

			for (int32 Idx = 0; Idx < 4; ++Idx)
			{
				const int32 SortedIdx = Indexes[Idx];

				Vertices[Idx] = VerticesCopy[SortedIdx];
				UVs[Idx] = UVsCopy[SortedIdx];
			}
		}

		/**
		 * Compares FWallPanel UVs. They need to have been sorted by UVs prior to calling.
		 *
		 * @param Other The other FWallPanel that we are comparing with one with
		 *
		 * @return True if the are similar
		*/
		bool HasSimilarUVs(const FWallPanel& Other) const
		{
			for (int32 UVIdx = 0; UVIdx < 4; ++UVIdx)
			{
				for (int32 ComponentIdx = 0; ComponentIdx < 2; ++ComponentIdx)
				{
					if (!FMath::IsNearlyEqual(UVs[UVIdx].Component(ComponentIdx), Other.UVs[UVIdx].Component(ComponentIdx), 0.001f))
					{
						return false;
					}
				}
			}

			return true;
		}

		static constexpr int32 TopLeftIdx = 0;
		static constexpr int32 TopRightIdx = 1;
		static constexpr int32 BottomLeftIdx = 2;
		static constexpr int32 BottomRightIdx = 3;
	};

	/**
	 * Sorts the given array of panels in left-right then top-bottom order, and assigns found
	 * row/col values to Panel structures.
	 *
	 * @param InOutWallPanels Array of panels that will be sorted.
	 */
	void SortWallPanelsInRowsAndColumns(TArray<FWallPanel>& InOutWallPanels)
	{
		// First sort in left-right then top-bottom order
		InOutWallPanels.Sort([&](const FWallPanel& PanelA, const FWallPanel& PanelB) -> bool
			{
				// This row factor should be ok because we're sorting UVs that are not expected to go over 1.0.
				constexpr float UVYFactor = 100.0f;

				const float ValueA = PanelA.UVs[FWallPanel::TopLeftIdx].X + UVYFactor * PanelA.UVs[FWallPanel::TopLeftIdx].Y;
				const float ValueB = PanelB.UVs[FWallPanel::TopLeftIdx].X + UVYFactor * PanelB.UVs[FWallPanel::TopLeftIdx].Y;

				return ValueA < ValueB;
			});

		// Estimate the number of columns per row
		int32 NumColumns = 1;

		for (int32 PanelIdx = 1; PanelIdx < InOutWallPanels.Num(); ++PanelIdx)
		{
			const float UVx[2] = {
				InOutWallPanels[PanelIdx - 1].UVs[FWallPanel::TopLeftIdx].X,
				InOutWallPanels[PanelIdx - 0].UVs[FWallPanel::TopLeftIdx].X
			};

			if ((UVx[1] < UVx[0]) || FMath::IsNearlyEqual(UVx[1], UVx[0]))
			{
				break;
			}

			NumColumns++;
		}

		// Now assign the estimated row/column values in the panel structure
		int32 Row = 0;
		int32 Col = 0;

		for (FWallPanel& Panel : InOutWallPanels)
		{
			Panel.Row = Row;
			Panel.Col = Col;

			if (++Col == NumColumns)
			{
				Col = 0;
				Row++;
			}
		}
	}

	/**
	 * Finds square panels in the mesh and populates the given array of wall panels with what it finds.
	 *
	 * @param OutWallPanels Array of panels to be populated
	 * @param MeshDescription description of the mesh containing the panels.
	 */
	void FindWallPanelsFromMeshDescription(TArray<FWallPanel>& OutWallPanels, FMeshDescription* MeshDescription)
	{
		// We query the MeshDescription via named attributes
		FStaticMeshAttributes Attributes(*MeshDescription); // MeshDescription can't be const here

		// Vertex 3d positions, indexable by VertexId
		const TVertexAttributesRef<const FVector3f> VertexPositions = MeshDescription->VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);

		// UV 2d positions, indexable by VertexInstaceId
		const TVertexInstanceAttributesRef<const FVector2f> VertexInstanceUVs = MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);

		// Iterate over all the triangles
		for (const FTriangleID& TriangleElementId : MeshDescription->Triangles().GetElementIDs())
		{
			// Get the 3 vertex instance ids of the triangle
			const FVertexInstanceID& VI0 = MeshDescription->GetTriangleVertexInstance(TriangleElementId, 0);
			const FVertexInstanceID& VI1 = MeshDescription->GetTriangleVertexInstance(TriangleElementId, 1);
			const FVertexInstanceID& VI2 = MeshDescription->GetTriangleVertexInstance(TriangleElementId, 2);

			// Get the 3 vertex ids of the triangle
			const FVertexID& V0 = MeshDescription->GetVertexInstanceVertex(VI0);
			const FVertexID& V1 = MeshDescription->GetVertexInstanceVertex(VI1);
			const FVertexID& V2 = MeshDescription->GetVertexInstanceVertex(VI2);

			FWallPanel WallPanel;

			WallPanel.Vertices[0] = (FVector)VertexPositions[V0];
			WallPanel.Vertices[1] = (FVector)VertexPositions[V1];
			WallPanel.Vertices[2] = (FVector)VertexPositions[V2];

			WallPanel.UVs[0] = VertexInstanceUVs[VI0];
			WallPanel.UVs[1] = VertexInstanceUVs[VI1];
			WallPanel.UVs[2] = VertexInstanceUVs[VI2];

			// Now we just need to find the 4th vertex/uv. 
			// In this algorithm, we assume they are rectangular. So it is a matter of adding the 2 vectors.

			const float DistanceUV01 = FVector2f::Distance(WallPanel.UVs[0], WallPanel.UVs[1]);
			const float DistanceUV02 = FVector2f::Distance(WallPanel.UVs[0], WallPanel.UVs[2]);
			const float DistanceUV12 = FVector2f::Distance(WallPanel.UVs[1], WallPanel.UVs[2]);

			const float DistanceMax = FMath::Max(DistanceUV01, FMath::Max(DistanceUV02, DistanceUV12));

			if (FMath::IsNearlyEqual(DistanceMax, DistanceUV01))
			{
				const FVector2f& OriginUV = WallPanel.UVs[2];
				const FVector& Origin3d = WallPanel.Vertices[2];

				WallPanel.UVs[3] = OriginUV + (WallPanel.UVs[0] - OriginUV) + (WallPanel.UVs[1] - OriginUV);
				WallPanel.Vertices[3] = Origin3d + (WallPanel.Vertices[0] - Origin3d) + (WallPanel.Vertices[1] - Origin3d);
			}
			else if (FMath::IsNearlyEqual(DistanceMax, DistanceUV02))
			{
				const FVector2f& OriginUV = WallPanel.UVs[1];
				const FVector& Origin3d = WallPanel.Vertices[1];

				WallPanel.UVs[3] = OriginUV + (WallPanel.UVs[0] - OriginUV) + (WallPanel.UVs[2] - OriginUV);
				WallPanel.Vertices[3] = Origin3d + (WallPanel.Vertices[0] - Origin3d) + (WallPanel.Vertices[2] - Origin3d);
			}
			else
			{
				const FVector2f& OriginUV = WallPanel.UVs[0];
				const FVector& Origin3d = WallPanel.Vertices[0];

				WallPanel.UVs[3] = OriginUV + (WallPanel.UVs[1] - OriginUV) + (WallPanel.UVs[2] - OriginUV);
				WallPanel.Vertices[3] = Origin3d + (WallPanel.Vertices[1] - Origin3d) + (WallPanel.Vertices[2] - Origin3d);
			}

			WallPanel.SortByUVs();

			bool bIsNewWallPanel = true;

			for (FWallPanel& ExistingWallPanel : OutWallPanels)
			{
				if (WallPanel.HasSimilarUVs(ExistingWallPanel))
				{
					bIsNewWallPanel = false;
					break;
				}
			}

			if (bIsNewWallPanel)
			{
				OutWallPanels.Add(WallPanel);
			}
		}
	}

#if WITH_OPENCV

	/**
	 * Structure to keep information about an Aruco Dictionary
	 */
	struct FArucoDictionaryInfo
	{
		/** Name of the dictionary */
		FString Name;

		/** Pointer to the Aruco dictionary data structure */
		cv::Ptr<cv::aruco::Dictionary> Dict;
	};

	/**
	 * Retrieves found Aruco dictionaries
	 *
	 * @return A const ref to the array of the available aruco dictionaries.
	 */
	static const TArray<FArucoDictionaryInfo>& GetArucoDictionariesRef()
	{
		static TArray<FArucoDictionaryInfo> Dictionaries =
		{
			BUILD_ARUCO_INFO(DICT_4X4_50),
			BUILD_ARUCO_INFO(DICT_4X4_100),
			BUILD_ARUCO_INFO(DICT_4X4_250),
			BUILD_ARUCO_INFO(DICT_4X4_1000),
			BUILD_ARUCO_INFO(DICT_5X5_50),
			BUILD_ARUCO_INFO(DICT_5X5_100),
			BUILD_ARUCO_INFO(DICT_5X5_250),
			BUILD_ARUCO_INFO(DICT_5X5_1000),
			BUILD_ARUCO_INFO(DICT_6X6_50),
			BUILD_ARUCO_INFO(DICT_6X6_100),
			BUILD_ARUCO_INFO(DICT_6X6_250),
			BUILD_ARUCO_INFO(DICT_6X6_1000),
			BUILD_ARUCO_INFO(DICT_7X7_50),
			BUILD_ARUCO_INFO(DICT_7X7_100),
			BUILD_ARUCO_INFO(DICT_7X7_250),
			BUILD_ARUCO_INFO(DICT_7X7_1000),
			BUILD_ARUCO_INFO(DICT_ARUCO_ORIGINAL),
		};

		return Dictionaries;
	}
#endif //WITH_OPENCV
};

USceneComponent* FLedWallCalibration::GetTypedParentComponent(const USceneComponent* InComponent, const UClass* InParentClass)
{
	if (!InComponent || !InParentClass)
	{
		return nullptr;
	}

	if (USceneComponent* ParentComponent = InComponent->GetAttachParent())
	{
		return ParentComponent->IsA(InParentClass) ? ParentComponent : nullptr;
	}

	// If we're here, the owner could be a BP generated class.

	UBlueprintGeneratedClass* BPGeneratedClass = InComponent->GetTypedOuter<UBlueprintGeneratedClass>();

	if (!BPGeneratedClass)
	{
		return nullptr;
	}

	TArray<const UBlueprintGeneratedClass*> BlueprintClasses;

	UBlueprintGeneratedClass::GetGeneratedClassesHierarchy(BPGeneratedClass, BlueprintClasses);

	for (const UBlueprintGeneratedClass* BPGClass : BlueprintClasses)
	{
		if (!BPGClass->SimpleConstructionScript)
		{
			continue;
		}

		const TArray<USCS_Node*> CDONodes = BPGClass->SimpleConstructionScript->GetAllNodes();

		for (const USCS_Node* Node : CDONodes)
		{
			for (const USCS_Node* ChildNode : Node->ChildNodes)
			{
				const UActorComponent* CDOComponent 
					= ChildNode->GetActualComponentTemplate(Cast<UBlueprintGeneratedClass>(BPGeneratedClass));

				// If the child is the InComponent, then we've found our parent component.
				if (CDOComponent == InComponent)
				{
					USceneComponent* ParentComponent 
						= Cast<USceneComponent>(Node->GetActualComponentTemplate(Cast<UBlueprintGeneratedClass>(BPGeneratedClass)));

					if (ParentComponent && ParentComponent->IsA(InParentClass))
					{
						return ParentComponent;
					}

					return nullptr;
				}
			}
		}
	}

	// If we're here, the component might be under an inherited component, so we search for it by its variable name.

	// Find the expected parent variable name

	FName ExpectedParentVariableName(NAME_None);
	{
		bool bFoundExpectedParentVariableName = false;

		for (const UBlueprintGeneratedClass* BPGClass : BlueprintClasses)
		{
			if (!BPGClass->SimpleConstructionScript)
			{
				continue;
			}

			const TArray<USCS_Node*> CDONodes = BPGClass->SimpleConstructionScript->GetAllNodes();

			for (const USCS_Node* Node : CDONodes)
			{
				const UActorComponent* NodeCDOComponent
					= Node->GetActualComponentTemplate(Cast<UBlueprintGeneratedClass>(BPGeneratedClass));

				if (NodeCDOComponent == InComponent)
				{
					ExpectedParentVariableName = Node->ParentComponentOrVariableName;
					bFoundExpectedParentVariableName = true;
					break;
				}
			}

			if (bFoundExpectedParentVariableName)
			{
				break;
			}
		}

		if (!bFoundExpectedParentVariableName || (ExpectedParentVariableName == NAME_None))
		{
			return nullptr;
		}
	}

	// Traverse the components until one with the expected parent variable name is found

	for (const UBlueprintGeneratedClass* BPGClass : BlueprintClasses)
	{
		if (!BPGClass->SimpleConstructionScript)
		{
			continue;
		}

		const TArray<USCS_Node*> CDONodes = BPGClass->SimpleConstructionScript->GetAllNodes();

		for (const USCS_Node* Node : CDONodes)
		{
			USceneComponent* NodeCDOComponent
				= Cast<USceneComponent>(Node->GetActualComponentTemplate(Cast<UBlueprintGeneratedClass>(BPGeneratedClass)));

			if (NodeCDOComponent == InComponent)
			{
				continue;
			}

			if (NodeCDOComponent && NodeCDOComponent->IsA(InParentClass))
			{
				if (Node->GetVariableName() == ExpectedParentVariableName)
				{
					return NodeCDOComponent;
				}
			}
		}
	}

	// If we're here, we could not find the parent component (of the expected class).
	return nullptr;
}

bool FLedWallCalibration::GenerateArucosForCalibrationPoint(
	UCalibrationPointComponent* CalibrationPoint,
	const FLedWallArucoGenerationOptions& Options,
	int32& OutNextMarkerId,
	cv::Mat& OutMat)
{
#if WITH_OPENCV && WITH_EDITORONLY_DATA

	// Note: UStaticMesh::GetMeshDescription is only available WITH_EDITORONLY_DATA

	using namespace LedWallCalibration;

	if (!CalibrationPoint)
	{
		UE_LOGF(LogLedWallCalibration, Error, "Invalid CalibrationPointComponent.");
		return false;
	}

	// Find its parent mesh
	const UStaticMeshComponent* ParentMeshComponent = GetTypedParentComponent<UStaticMeshComponent>(CalibrationPoint);

	if (!ParentMeshComponent)
	{
		UE_LOGF(LogLedWallCalibration, Error, "Calibration point did not have a static mesh component parent.  "
			"This is required because the calibration points must stay with the mesh in case the mesh gets moved."
		);

		return false;
	}

	const UStaticMesh* StaticMesh = ParentMeshComponent->GetStaticMesh();

	if (!StaticMesh)
	{
		UE_LOGF(LogLedWallCalibration, Error, "ParentMeshComponent did not have a StaticMesh in it.");
		return false;
	}

	FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);

	if (!MeshDescription)
	{
		UE_LOGF(LogLedWallCalibration, Error, "Mesh did not have a valid Mesh Description");
		return false;
	}

	// Here we'll store the positional information needed to generate the Arucos
	TArray<FWallPanel> WallPanels;

	FindWallPanelsFromMeshDescription(WallPanels, MeshDescription);

	if (MeshDescription->Triangles().Num() % 2)
	{
		UE_LOGF(LogLedWallCalibration, Error, "Odd. The number of triangles (%d) was expected to be an even number.",
			MeshDescription->Triangles().Num()
		);

		return false;
	}

	const int32 ExpectedNumPanels = MeshDescription->Triangles().Num() / 2;

	if (WallPanels.Num() != ExpectedNumPanels) // This indirectly verifies the calculated 4th uv/vertex
	{
		UE_LOGF(LogLedWallCalibration, Error, "Panel detection failed because its detected amount (%d) does not equal half the number of triangles (%d)",
			WallPanels.Num(),
			ExpectedNumPanels
		);

		return false;
	}

	SortWallPanelsInRowsAndColumns(WallPanels);

	// Create the aruco calibration subpoints and the corresponding texture. Initialize with Black.

	if (Options.TextureWidth < 1 || Options.TextureHeight < 1)
	{
		UE_LOGF(LogLedWallCalibration, Error, "Invalid texture dimensions of %d x %d",
			Options.TextureWidth,
			Options.TextureHeight
		);

		return false;
	}

	cv::Mat ArucosMat(cv::Size(Options.TextureWidth, Options.TextureHeight), CV_8UC1, cv::Scalar(0));

	//@todo get dictionary name from dialog
	const FString DesiredDictionaryName = Options.ArucoDictionaryAsString();

	FArucoDictionaryInfo DictionaryInfo;

	for (const FArucoDictionaryInfo& DictInfo : GetArucoDictionariesRef())
	{
		if (DictInfo.Name == DesiredDictionaryName)
		{
			DictionaryInfo = DictInfo;
			break;
		}
	}

	if (DictionaryInfo.Dict.empty())
	{
		UE_LOGF(LogLedWallCalibration, Error, "Invalid Aruco dictionary");
		return false;
	}

	int32 MarkerId = Options.MarkerId;

	if (MarkerId < 1)
	{
		UE_LOGF(LogLedWallCalibration, Error, "Invalid starting Marker Id (%d)",
			MarkerId
		);

		return false;
	}

	// check that the dictionary will have enough symbols
	{
		const int32 NumAvailableSymbols = DictionaryInfo.Dict->bytesList.rows - MarkerId + 1;

		if (NumAvailableSymbols < WallPanels.Num())
		{
			UE_LOGF(LogLedWallCalibration, Error, "Not enough symbols in Aruco dictionary. The number of detected panels is %d, "
				"but the number of symbols after and including the starting marked id #%d is %d",
				WallPanels.Num(),
				MarkerId,
				NumAvailableSymbols
			);
			return false;
		}
	}

#if WITH_EDITOR
	CalibrationPoint->Modify();
#endif //WITH_EDITOR

	// The calibration point itself should be at the origin of the mesh it is parented to.
	CalibrationPoint->SetRelativeTransform(FTransform::Identity);

	// The subpoints will contain the named Arucos
	CalibrationPoint->SubPoints.Empty();

	// The order of CornerNames must match how the UVs were sorted in the FWallPanel
	TArray<FString, TInlineAllocator<4>> CornerNames;
	{
		CornerNames.AddDefaulted(4);

		CornerNames[FWallPanel::TopLeftIdx]     = TEXT("TL");
		CornerNames[FWallPanel::TopRightIdx]    = TEXT("TR");
		CornerNames[FWallPanel::BottomLeftIdx]  = TEXT("BL");
		CornerNames[FWallPanel::BottomRightIdx] = TEXT("BR");
	}

	for (FWallPanel& WallPanel : WallPanels)
	{
		if ((Options.PlaceModulus > 1) && ((WallPanel.Row + WallPanel.Col) % Options.PlaceModulus))
		{
			continue;
		}

		const int32 PanelWidthPixels
			= int32(FMath::RoundHalfFromZero(ArucosMat.cols * (WallPanel.UVs[FWallPanel::BottomRightIdx].X - WallPanel.UVs[FWallPanel::TopLeftIdx].X)));

		const int32 PanelHeightPixels
			= int32(FMath::RoundHalfFromZero(ArucosMat.rows * (WallPanel.UVs[FWallPanel::BottomRightIdx].Y - WallPanel.UVs[FWallPanel::TopLeftIdx].Y)));

		if (PanelWidthPixels < 1 || PanelHeightPixels < 1)
		{
			UE_LOGF(LogLedWallCalibration, Warning, "Skipping invalid panel dimensions of %d x %d",
				PanelWidthPixels,
				PanelHeightPixels
			);

			continue;
		}

		const int32 MarkerBits = DictionaryInfo.Dict->markerSize;
		const int32 TotalBits = MarkerBits + 2;

		if (MarkerBits % 2)
		{
			UE_LOGF(LogLedWallCalibration, Warning, "MarkerBits (%d) was not an even number, so the marker won't be perfectly centered",
				MarkerBits
			);
		}

		const float WidthPixelsPerBitFloat = float(PanelWidthPixels) / float(TotalBits);
		const int32 WidthPixelsPerBit = FMath::RoundHalfFromZero(WidthPixelsPerBitFloat);

		const float HeightPixelsPerBitFloat = float(PanelHeightPixels) / float(TotalBits);
		const int32 HeightPixelsPerBit = FMath::RoundHalfFromZero(HeightPixelsPerBitFloat);

		// The Arucos need to be square, so we take the smallest side of the panel for them.
		
		const int32 MarkerPixelsPerBit = FMath::Min(WidthPixelsPerBit, HeightPixelsPerBit);

		const int32 MarkerSidePixels = MarkerPixelsPerBit * MarkerBits;
		const int32 MarkerBorderWidthBits = 1;

		// Draw marker onto its own Mat
		cv::Mat ArucoMat;
		cv::aruco::drawMarker(DictionaryInfo.Dict, MarkerId, MarkerSidePixels, ArucoMat, MarkerBorderWidthBits);

		// Draw panel onto its own Mat, initialized with White.
		cv::Mat PanelMat(cv::Size(PanelWidthPixels, PanelHeightPixels), CV_8UC1, cv::Scalar(255));

		// Copy marker mat onto panel mat

		const FIntPoint ArucoPixelsTL(
			(PanelWidthPixels - MarkerSidePixels) / 2,
			(PanelHeightPixels - MarkerSidePixels) / 2
		);

		ArucoMat.copyTo(PanelMat(cv::Rect(ArucoPixelsTL[0], ArucoPixelsTL[1], MarkerSidePixels, MarkerSidePixels)));

		// Copy panel onto output Mat
		{
			const int32 LeftPixel = int32(FMath::RoundHalfFromZero(ArucosMat.cols * WallPanel.UVs[FWallPanel::TopLeftIdx].X));
			const int32 TopPixel  = int32(FMath::RoundHalfFromZero(ArucosMat.rows * WallPanel.UVs[FWallPanel::TopLeftIdx].Y));

			PanelMat.copyTo(ArucosMat(cv::Rect(LeftPixel, TopPixel, PanelMat.cols, PanelMat.rows)));
		}

		// We need to create a subpoint for every Aruco corner (4 per panel)
		// e.g. name:
		//     DICT_6X6_250-23-TL
		//
		{
			const FIntPoint ArucoPixelsBR = ArucoPixelsTL + FIntPoint(ArucoMat.cols, ArucoMat.rows);

			FVector2D ArucoInPanelUVs[4];

			ArucoInPanelUVs[FWallPanel::TopLeftIdx]     = FVector2D(float(ArucoPixelsTL.X) / PanelMat.cols, float(ArucoPixelsTL.Y) / PanelMat.rows);
			ArucoInPanelUVs[FWallPanel::TopRightIdx]    = FVector2D(float(ArucoPixelsBR.X) / PanelMat.cols, float(ArucoPixelsTL.Y) / PanelMat.rows);
			ArucoInPanelUVs[FWallPanel::BottomLeftIdx]  = FVector2D(float(ArucoPixelsTL.X) / PanelMat.cols, float(ArucoPixelsBR.Y) / PanelMat.rows);
			ArucoInPanelUVs[FWallPanel::BottomRightIdx] = FVector2D(float(ArucoPixelsBR.X) / PanelMat.cols, float(ArucoPixelsBR.Y) / PanelMat.rows);

			TArray<FString, TInlineAllocator<4>> PointNames;

			for (FString& CornerName : CornerNames)
			{
				PointNames.Add(FString::Printf(TEXT("%s-%d-%s"), *DictionaryInfo.Name, MarkerId, *CornerName));
			}

			check(PointNames.Num() == 4);

			const FVector PanelVectorX(WallPanel.Vertices[FWallPanel::TopRightIdx]   - WallPanel.Vertices[FWallPanel::TopLeftIdx]);
			const FVector PanelVectorY(WallPanel.Vertices[FWallPanel::BottomLeftIdx] - WallPanel.Vertices[FWallPanel::TopLeftIdx]);

			for (int32 Idx = 0; Idx < 4; ++Idx)
			{
				const FVector ArucoCorner =
					WallPanel.Vertices[FWallPanel::TopLeftIdx] + ArucoInPanelUVs[Idx].X * PanelVectorX + ArucoInPanelUVs[Idx].Y * PanelVectorY;

				CalibrationPoint->SubPoints.Add(PointNames[Idx], ArucoCorner);
			}
		}

		// Increment marker id since all markers should be different
		MarkerId++;
	}

	// Rebuild the CalibrationPoint mesh for proper display
	CalibrationPoint->RebuildVertices();

	OutNextMarkerId = MarkerId;

	OutMat = ArucosMat;

	return true;
#else

	UE_LOGF(LogLedWallCalibration, Error, "OpenCV is required for Aruco support");
	return false;

#endif //WITH_OPENCV
}

#undef LOCTEXT_NAMESPACE
