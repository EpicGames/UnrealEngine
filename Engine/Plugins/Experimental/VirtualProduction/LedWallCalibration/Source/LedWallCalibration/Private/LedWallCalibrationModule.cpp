// Copyright Epic Games, Inc. All Rights Reserved.

#include "LedWallCalibrationLog.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "LedWallCalibration"

DEFINE_LOG_CATEGORY(LogLedWallCalibration);

class FLedWallCalibrationModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FLedWallCalibrationModule, LedWallCalibration)
