// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "OpenCVHelper.h"

#include "LedWallArucoGenerationOptions.generated.h"

/**
 * Structure that can be passed to the Aruco generation function
 */
USTRUCT(BlueprintType)
struct FLedWallArucoGenerationOptions
{
	GENERATED_BODY()

public:

	/** Width of the texture that will contain the Aruco markers */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Aruco")
	int32 TextureWidth = 3840;

	/** Height of the texture that will contain the Aruco markers */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Aruco")
	int32 TextureHeight = 2160;

	/** Aruco dictionary to use when generating the markers */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Aruco")
	EArucoDictionary ArucoDictionary = EArucoDictionary::DICT_6X6_1000;

	/** Starting marker Id. Arucos will be generated with consecutive Marker Ids, starting from this one */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Aruco")
	int32 MarkerId = 1;

	/** 
	 * Used to avoid using up the symbols in the dictionary as quickly.
	 * Will place the next marker id when [(row + column) mod PlaceModulus] is zero.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Aruco")
	int32 PlaceModulus = 2;

public:

	/**
	 * Returns name of selected Aruco dictionary 
	 * 
	 * @return Name of selected Aruco dictionary
	 */
	FString ArucoDictionaryAsString() const
	{
		const UEnum* EnumPtr = FindObject<UEnum>(nullptr, TEXT("/Script/OpenCVHelper.EArucoDictionary"), EFindObjectFlags::ExactClass);
		check(EnumPtr);

#if WITH_EDITOR
		return EnumPtr->GetDisplayNameTextByIndex(static_cast<uint8>(ArucoDictionary)).ToString();
#else
		return EnumPtr->GetNameStringByIndex(static_cast<uint8>(ArucoDictionary));
#endif
	}
};
