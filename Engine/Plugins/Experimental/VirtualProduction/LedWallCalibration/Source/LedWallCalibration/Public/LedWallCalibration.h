// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UCalibrationPointComponent;
class UStaticMeshComponent;
class USceneComponent;

struct FLedWallArucoGenerationOptions;

namespace cv
{
	class Mat;
};

class LEDWALLCALIBRATION_API FLedWallCalibration
{

public:

	/**
	 * Populates calibration points and generates a texture with Aruco markers. It finds the CalibrationPointComponent's parent,
	 * which is expected to be a StaticMeshComponent, and finds the led panels in it (it expects its triangles to have 3 corners of
	 * any given panel. It then generates an Aruco marker for each detected panel and populates its named calibration point corners.
	 *
	 * @param InCalibrationPoint The calibration point that will be populated.
	 * @param InOptions The options for Aruco generation.
	 * @param OutNextMarkerId If you re-run this function for another mesh, use OutNextMarkerId in the input options.
	 * @param OutMat The generated OpenCV Mat with the Aruco markers.
	 *
	 * @return True if successful
	 */
	static bool GenerateArucosForCalibrationPoint(
		UCalibrationPointComponent* CalibrationPoint,
		const FLedWallArucoGenerationOptions& Options,
		int32& OutNextMarkerId,
		cv::Mat& OutMat);


	/**
	 * Finds the parent component of the given component.
	 *
	 * @param InComponent Component that we are looking the parent for.
	 * @param ParentClass Parent class to search for.
	 *
	 * @return The found component that InComponent is parented to.
	 */
	static USceneComponent* GetTypedParentComponent(const USceneComponent* InComponent, const UClass* InParentClass);

	/**
	 * Finds the parent component of the given calibration point component
	 *
	 * @param InComponent Component that we are looking the parent for.
	 *
	 * @return The found component that InComponent is parented to.
	 */
	template <class T>
	static T* GetTypedParentComponent(const USceneComponent* InComponent)
	{
		return (T*)GetTypedParentComponent(InComponent, T::StaticClass());
	}
};
