// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LedWallCalibrationEditor : ModuleRules
{
	public LedWallCalibrationEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		OptimizeCode = CodeOptimization.Never;
		bUseUnity = false;
		PCHUsage = PCHUsageMode.NoPCHs;

		DefaultBuildSettings = BuildSettingsVersion.Latest;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AssetRegistry",
				"AssetTools",
				"CameraCalibrationCore",
				"CameraCalibrationCoreEditor",
				"Core",
				"CoreUObject",
				"Engine",
				"LedWallCalibration",
				"OpenCV",
				"OpenCVHelper",
				"Settings",
				"Slate",
				"SlateCore",
				"ToolWidgets",
				"UnrealEd",
			}
		);

		ShortName = "LedWCalEd";
	}
}
