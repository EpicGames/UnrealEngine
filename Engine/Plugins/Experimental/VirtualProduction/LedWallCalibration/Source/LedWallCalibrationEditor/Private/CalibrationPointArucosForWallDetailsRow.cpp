// Copyright Epic Games, Inc. All Rights Reserved.

#include "CalibrationPointArucosForWallDetailsRow.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "CalibrationPointComponent.h"
#include "Components/StaticMeshComponent.h"
#include "DetailWidgetRow.h"
#include "DetailsViewArgs.h"
#include "Dialog/SCustomDialog.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Texture2D.h"
#include "IAssetTools.h"
#include "IStructureDetailsView.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "LedWallCalibration.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif //WITH_EDITOR


#if WITH_OPENCV
// IWYU pragma: begin_keep
#include "OpenCVHelper.h"
#include "PreOpenCVHeaders.h"

#include "opencv2/imgproc.hpp"

#include "PostOpenCVHeaders.h"
// IWYU pragma: end_keep
#endif //WITH_OPENCV


#define LOCTEXT_NAMESPACE "CalibrationPointArucosForWallDetailsRow"


FText FCalibrationPointArucosForWallDetailsRow::GetSearchString() const
{
	return LOCTEXT("CreateArucos", "Create Arucos");
}

bool FCalibrationPointArucosForWallDetailsRow::IsAdvanced() const
{
	return false;
}

void FCalibrationPointArucosForWallDetailsRow::CreateArucos(
	const TArray<TWeakObjectPtr<UCalibrationPointComponent>>& SelectedCalibrationPointComponents)
{
#if WITH_OPENCV

	if (!SelectedCalibrationPointComponents.Num())
	{
		return;
	}

	// Pick first calibration point component
	UCalibrationPointComponent* CalibrationPoint = SelectedCalibrationPointComponents[0].Get();

	TSharedPtr<TStructOnScope<FLedWallArucoGenerationOptions>> ArucoGenerationOptions
		= MakeShared<TStructOnScope<FLedWallArucoGenerationOptions>>();

	ArucoGenerationOptions->InitializeAs<FLedWallArucoGenerationOptions>();

	FLedWallArucoGenerationOptions& Options = *ArucoGenerationOptions->Get();

	// Pre-fill some options with best guesses
	{
		// Next marker id should be the next one after the last one used
		Options.MarkerId = PreviousNextMarkerId;

		// By default, use the same dictionary as before.
		Options.ArucoDictionary = PreviousArucoDictionaryUsed;
	}

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	FStructureDetailsViewArgs StructureViewArgs;
	FDetailsViewArgs DetailArgs;

	DetailArgs.bAllowSearch = false;
	DetailArgs.bShowScrollBar = true;

	TSharedPtr<IStructureDetailsView> ArucoGenerationOptionsDetailsView 
		= PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, ArucoGenerationOptions);

	TSharedRef<SCustomDialog> OptionsWindow =
		SNew(SCustomDialog)
		.Title(LOCTEXT("ArucoGenerationOptions", "Aruco Generation Options"))
		.Content()
		[
			ArucoGenerationOptionsDetailsView->GetWidget().ToSharedRef()
		]
		.Buttons
		({
			SCustomDialog::FButton(LOCTEXT("Ok", "Ok")),
			SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel")),
		});

	const int32 PressedButtonIdx = OptionsWindow->ShowModal();

	if (PressedButtonIdx != 0)
	{
		return;
	}

	cv::Mat ArucoMat;

#if WITH_EDITOR
	FScopedTransaction Transaction(LOCTEXT("GenerateArucoMarkers", "Generate Aruco Markers"));
#endif //WITH_EDITOR

	const bool bGeneratedArucos = FLedWallCalibration::GenerateArucosForCalibrationPoint(
		CalibrationPoint, 
		Options, 
		PreviousNextMarkerId,
		ArucoMat
	);

	if (!bGeneratedArucos)
	{
		const FText Message = LOCTEXT("ErrorWhenGeneratingArucos", 
			"There was an error when generatic the Aruco markers for this mesh.\n"
			"Please read the Output Log for more details."
		);

		const FText Title = LOCTEXT("ArucoGenerationError", "Aruco Generation Error");

		FMessageDialog::Open(EAppMsgType::Ok, Message, Title);

		return;
	}

	PreviousArucoDictionaryUsed = Options.ArucoDictionary;

#if WITH_EDITOR
	// In case this is a BPGC, mark it as modified to force recompile
	if (UBlueprintGeneratedClass* BPGC = CalibrationPoint->GetTypedOuter<UBlueprintGeneratedClass>())
	{
		if (UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(BPGC))
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
#endif //WITH_EDITOR

	// Save texture
	{
		const UStaticMeshComponent* ParentMesh = FLedWallCalibration::GetTypedParentComponent<UStaticMeshComponent>(CalibrationPoint);
		FString ParentMeshName;
		
		if (ParentMesh)
		{
			ParentMeshName = ParentMesh->GetName();

			// If the mesh component is an arquetype, it will have a _GEN_VARIABLE suffix. 
			// We don't want it in the texture name, so we prune it.

			const FString GenVariable(TEXT("_GEN_VARIABLE"));

			if (ParentMeshName.EndsWith(GenVariable))
			{
				ParentMeshName = ParentMeshName.LeftChop(GenVariable.Len());
			}
		}
		else
		{
			ParentMeshName =  TEXT("Mesh");
		}

		const FString NewNameSuggestion = FString::Printf(
			TEXT("T_%s_%s_%dx%d_mk%d-%d"),
			*ParentMeshName,
			*Options.ArucoDictionaryAsString(),
			Options.TextureWidth,
			Options.TextureHeight,
			Options.MarkerId,
			PreviousNextMarkerId-1
		);

		FString PackageName = FString(TEXT("/Game/Textures/")) + NewNameSuggestion;
		FString Name;
			
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().CreateUniqueAssetName(PackageName, TEXT(""), PackageName, Name);
			
		TSharedPtr<SDlgPickAssetPath> PickAssetPathWidget =
			SNew(SDlgPickAssetPath)
			.Title(LOCTEXT("SaveArucoTextureForMesh", "Choose New Aruco Texture Location"))
			.DefaultAssetPath(FText::FromString(PackageName));

		if (PickAssetPathWidget->ShowModal() == EAppReturnType::Ok)
		{
			// Get the full name of where we want to create the physics asset.
			FString UserPackageName = PickAssetPathWidget->GetFullAssetPath().ToString();
			FName TextureName(*FPackageName::GetLongPackageAssetName(UserPackageName));

			// Check if the user inputed a valid asset name, if they did not, give it the generated default name
			if (TextureName == NAME_None)
			{
				// Use the defaults that were already generated.
				UserPackageName = PackageName;
				TextureName = *Name;
			}

			// Convert to RGBA before generating the texture to avoid other plugins from showing Red instead of gray.
			if (ArucoMat.channels() == 1)
			{
				cv::cvtColor(ArucoMat, ArucoMat, cv::COLOR_GRAY2RGBA);
			}

			UTexture* ArucoTexture = FOpenCVHelper::TextureFromCvMat(ArucoMat, &UserPackageName, &TextureName);

			if (!ArucoTexture)
			{
				const FText Message = LOCTEXT("ErrorWhenSavingArucoTexture", "There was an error when saving the texture with the Aruco markers.");
				const FText Title = LOCTEXT("ArucoTextureGenerationError", "Aruco Texture Generation Error");
				FMessageDialog::Open(EAppMsgType::Ok, Message, Title);

				return;
			}

			FAssetRegistryModule::AssetCreated(ArucoTexture);
			ArucoTexture->MarkPackageDirty();
		}
	}

	const FString Message = FString::Printf(TEXT(
		"Generated Aruco calibration points and texture using dictionary '%s'.\n"
		"\n"
		"If generating markers for another mesh in the wall, start from marker Id #%d."),
		*Options.ArucoDictionaryAsString(),
		PreviousNextMarkerId
	);

	const FText Title = LOCTEXT("ArucosGenerated", "Arucos Generated");
	FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(Message), Title);

#endif //WITH_OPENCV
}

void FCalibrationPointArucosForWallDetailsRow::CustomizeRow(
	FDetailWidgetRow& WidgetRow, 
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjectsList)
{
	TArray<TWeakObjectPtr<UCalibrationPointComponent>> SelectedCalibrationPointComponents;

	for (auto& Object : SelectedObjectsList)
	{
		if (UCalibrationPointComponent* CalibrationComponent = Cast<UCalibrationPointComponent>(Object.Get()))
		{
			SelectedCalibrationPointComponents.Add(TWeakObjectPtr<UCalibrationPointComponent>(CalibrationComponent));
		}
	}
	
	WidgetRow.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("LedWall", "LED Walls"))
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(SButton)
		.VAlign(VAlign_Center)
		.ToolTipText(LOCTEXT("ArucosForWallTooltip", "Finds parent mesh (or all meshes in actor if no parent mesh) and tries to add calibration points for the panels."))
		.OnClicked_Lambda([&, SelectedCalibrationPointComponents]()
			{
				CreateArucos(SelectedCalibrationPointComponents);
				return FReply::Handled();
			})
		.Content()
		[
			SNew(STextBlock)
			.Text(GetSearchString())
		]
	];
}

#undef LOCTEXT_NAMESPACE
