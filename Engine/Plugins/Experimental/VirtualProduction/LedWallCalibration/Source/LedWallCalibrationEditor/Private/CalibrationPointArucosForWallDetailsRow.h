// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ICalibrationPointComponentDetailsRow.h"
#include "LedWallArucoGenerationOptions.h"

class UCalibrationPointComponent;

class FCalibrationPointArucosForWallDetailsRow : public ICalibrationPointComponentDetailsRow
{
public:

	//~ Begin ICalibrationPointComponentDetailsRow interface
	virtual FText GetSearchString() const override;
	virtual bool IsAdvanced() const override;
	virtual void CustomizeRow(FDetailWidgetRow& WidgetRow, const TArray<TWeakObjectPtr<UObject>>& SelectedObjectsList) override;
	//~ End ICalibrationPointComponentDetailsRow interface

private:

	/** Finds the panel locations and adds calibration subpoints named after aruco markers to represent them */
	void CreateArucos(const TArray<TWeakObjectPtr<UCalibrationPointComponent>>& SelectedCalibrationPointComponents);

private:

	/** Remember last Aruco dictionary used so that it can be pre-chosen next time */
	EArucoDictionary PreviousArucoDictionaryUsed = EArucoDictionary::DICT_6X6_1000;

	/** 
	 * Remember next marker id so that it can pre-choose the next starting marker id. 
	 */
	int32 PreviousNextMarkerId = 1;
};
