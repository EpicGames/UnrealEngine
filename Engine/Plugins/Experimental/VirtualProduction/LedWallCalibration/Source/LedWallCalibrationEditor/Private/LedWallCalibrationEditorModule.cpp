// Copyright Epic Games, Inc. All Rights Reserved.


#include "CalibrationPointArucosForWallDetailsRow.h"
#include "CameraCalibrationCoreEditorModule.h"
#include "LedWallCalibrationEditorLog.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "LedWallCalibrationEditor"

DEFINE_LOG_CATEGORY(LogLedWallCalibrationEditor);

class FLedWallCalibrationEditorModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
#if WITH_OPENCV
		if (FApp::CanEverRender())
		{
			CalibrationPointArucosForWallDetailsRow = MakeShared<FCalibrationPointArucosForWallDetailsRow>();

			FCameraCalibrationCoreEditorModule& CameraCalibrationCoreEditorModule
				= FModuleManager::Get().LoadModuleChecked<FCameraCalibrationCoreEditorModule>("CameraCalibrationCoreEditor");

			CameraCalibrationCoreEditorModule.RegisterCalibrationPointDetailsRow(CalibrationPointArucosForWallDetailsRow);
		}
#endif //WITH_OPENCV
	}

	virtual void ShutdownModule() override
	{
#if WITH_OPENCV
		if (FApp::CanEverRender())
		{
			FCameraCalibrationCoreEditorModule& CameraCalibrationCoreEditorModule
				= FModuleManager::Get().LoadModuleChecked<FCameraCalibrationCoreEditorModule>("CameraCalibrationCoreEditor");

			CameraCalibrationCoreEditorModule.UnregisterCalibrationPointDetailsRow(CalibrationPointArucosForWallDetailsRow);
		}
#endif //WITH_OPENCV
	}

private:

	TSharedPtr<FCalibrationPointArucosForWallDetailsRow> CalibrationPointArucosForWallDetailsRow;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FLedWallCalibrationEditorModule, LedWallCalibrationEditor)
